{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animation Locomotion Library",
	"Description": "Collection of techniques for driving locomotion animations",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationLocomotionLibraryRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AnimationLocomotionLibraryEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}