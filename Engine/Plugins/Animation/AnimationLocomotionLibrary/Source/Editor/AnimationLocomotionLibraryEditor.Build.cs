// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimationLocomotionLibraryEditor : ModuleRules
	{
		public AnimationLocomotionLibraryEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
					"Core",
					"CoreUObject",
					"Engine",
					"AnimGraphRuntime",
					"AnimationModifiers",
					"AnimationBlueprintLibrary"
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
            });

			PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
			});

			ShortName = "AnimLLEd";
		}
	}
}
