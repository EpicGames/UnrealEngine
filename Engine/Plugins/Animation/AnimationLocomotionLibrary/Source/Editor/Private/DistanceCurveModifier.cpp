// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistanceCurveModifier.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "EngineLogs.h"

// TODO: This logic works decently for simple clips but it should be reworked to be more robust:
//  * It could detect pivot points by change in direction.
//  * It should also account for clips that have multiple stop/pivot points.
//  * It should handle distance traveled for the ends of looping animations.

#include UE_INLINE_GENERATED_CPP_BY_NAME(DistanceCurveModifier)
void UDistanceCurveModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "DistanceCurveModifier failed. Reason: Invalid Animation");
		return;
	}

	if (!Animation->HasRootMotion())
	{
		UE_LOGF(LogAnimation, Error, "DistanceCurveModifier failed. Reason: Root motion is disabled on the animation (%ls)", *GetNameSafe(Animation));
		return;
	}

	const bool bMetaDataCurve = false;
	UAnimationBlueprintLibrary::AddCurve(Animation, CurveName, ERawCurveTrackTypes::RCT_Float, bMetaDataCurve);

	const float AnimLength = Animation->GetPlayLength();
	float SampleInterval;
	int32 NumSteps;
	float TimeOfMinSpeed;

	if(bStopAtEnd)
	{ 
		TimeOfMinSpeed = AnimLength;
	}
	else
	{
		// Perform a high resolution search to find the sample point with minimum speed.
		
		TimeOfMinSpeed = 0.f;
		float MinSpeedSq = FMath::Square(StopSpeedThreshold);

		SampleInterval = 1.f / 120.f;
		NumSteps = AnimLength / SampleInterval;
		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			const float Time = Step * SampleInterval;

			const bool bAllowLooping = false;
			const FAnimExtractContext Context(static_cast<double>(Time), true, FDeltaTimeRecord(SampleInterval), bAllowLooping);
			const FVector RootMotionTranslation = Animation->ExtractRootMotion(Context).GetTranslation();
			const float RootMotionSpeedSq = CalculateMagnitudeSq(RootMotionTranslation, Axis) / SampleInterval;

			if (RootMotionSpeedSq < MinSpeedSq)
			{
				MinSpeedSq = RootMotionSpeedSq;
				TimeOfMinSpeed = Time;
			}
		}
	}

	SampleInterval = 1.f / SampleRate;
	NumSteps = FMath::CeilToInt(AnimLength / SampleInterval);
	float Time = 0.0f;
	for (int32 Step = 0; Step <= NumSteps && Time < AnimLength; ++Step)
	{
		Time = FMath::Min(Step * SampleInterval, AnimLength);

		// Assume that during any time before the stop/pivot point, the animation is approaching that point.
		// TODO: This works for clips that are broken into starts/stops/pivots, but needs to be rethought for more complex clips.
		const float ValueSign = (Time < TimeOfMinSpeed) ? -1.0f : 1.0f;

		const FVector RootMotionTranslation = Animation->ExtractRootMotionFromRange(TimeOfMinSpeed, Time, FAnimExtractContext()).GetTranslation();
		UAnimationBlueprintLibrary::AddFloatCurveKey(Animation, CurveName, Time, ValueSign * CalculateMagnitude(RootMotionTranslation, Axis));
	}
}

void UDistanceCurveModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	const bool bRemoveNameFromSkeleton = false;
	UAnimationBlueprintLibrary::RemoveCurve(Animation, CurveName, bRemoveNameFromSkeleton);
}

float UDistanceCurveModifier::CalculateMagnitude(const FVector& Vector, EDistanceCurve_Axis Axis)
{
	switch (Axis)
	{
	case EDistanceCurve_Axis::X:		return FMath::Abs(Vector.X);
	case EDistanceCurve_Axis::Y:		return FMath::Abs(Vector.Y);
	case EDistanceCurve_Axis::Z:		return FMath::Abs(Vector.Z);
	default: return FMath::Sqrt(CalculateMagnitudeSq(Vector, Axis));
	}
}

float UDistanceCurveModifier::CalculateMagnitudeSq(const FVector& Vector, EDistanceCurve_Axis Axis)
{
	switch (Axis)
	{
	case EDistanceCurve_Axis::X:		return FMath::Square(FMath::Abs(Vector.X));
	case EDistanceCurve_Axis::Y:		return FMath::Square(FMath::Abs(Vector.Y));
	case EDistanceCurve_Axis::Z:		return FMath::Square(FMath::Abs(Vector.Z));
	case EDistanceCurve_Axis::XY:		return Vector.X * Vector.X + Vector.Y * Vector.Y;
	case EDistanceCurve_Axis::XZ:		return Vector.X * Vector.X + Vector.Z * Vector.Z;
	case EDistanceCurve_Axis::YZ:		return Vector.Y * Vector.Y + Vector.Z * Vector.Z;
	case EDistanceCurve_Axis::XYZ:		return Vector.X * Vector.X + Vector.Y * Vector.Y + Vector.Z * Vector.Z;
	default: check(false); break;
	}

	return 0.f;
}
