// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimationModifier.h"
#include "DistanceCurveModifier.generated.h"

// @todo: Consolidate with EMotionExtractor_Axis
/** Axes to calculate the distance value from */
UENUM(BlueprintType)
enum class EDistanceCurve_Axis : uint8
{
	X,
	Y,
	Z,
	XY,
	XZ,
	YZ,
	XYZ
};

/** Extracts traveling distance information from root motion and bakes it to a curve.
 * A negative value indicates distance remaining to a stop or pivot point.
 * A positive value indicates distance traveled from a start point or from the beginning of the clip.
 */
UCLASS()
class UDistanceCurveModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	/** Rate used to sample the animation. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "1"))
	int32 SampleRate = 30;

	/** Name for the generated curve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName CurveName = "Distance";

	/** Root motion speed must be below this threshold to be considered stopped. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(EditCondition="!bStopAtEnd"))
	float StopSpeedThreshold = 5.0f;

	/** Axes to calculate the distance value from. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EDistanceCurve_Axis Axis = EDistanceCurve_Axis::XY;

	/** Root motion is considered to be stopped at the clip's end */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bStopAtEnd = false;

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;

private:

	/** Helper functions to calculate the magnitude of a vector only considering a specific axis or axes */
	static float CalculateMagnitude(const FVector& Vector, EDistanceCurve_Axis Axis);
	static float CalculateMagnitudeSq(const FVector& Vector, EDistanceCurve_Axis Axis);
};