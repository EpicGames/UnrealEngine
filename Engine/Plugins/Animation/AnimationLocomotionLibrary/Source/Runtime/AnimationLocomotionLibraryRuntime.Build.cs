// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimationLocomotionLibraryRuntime : ModuleRules
	{
		public AnimationLocomotionLibraryRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "AnimGraphRuntime",
                "AnimationCore",
                "Core",
                "CoreUObject",
                "Engine",
            });

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
            });

			ShortName = "AnimLLRun";
		}
	}
}
