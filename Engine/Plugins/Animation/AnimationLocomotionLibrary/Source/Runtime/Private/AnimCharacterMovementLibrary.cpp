// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimCharacterMovementLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCharacterMovementLibrary)

FVector UAnimCharacterMovementLibrary::PredictGroundMovementStopLocation(const FVector& Velocity, 
	bool bUseSeparateBrakingFriction, float BrakingFriction, float GroundFriction, float BrakingFrictionFactor, float BrakingDecelerationWalking)
{
	FVector PredictedStopLocation = FVector::ZeroVector;

	float ActualBrakingFriction = (bUseSeparateBrakingFriction ? BrakingFriction : GroundFriction);
	const float FrictionFactor = FMath::Max(0.f, BrakingFrictionFactor);
	ActualBrakingFriction = FMath::Max(0.f, ActualBrakingFriction * FrictionFactor);
	float BrakingDeceleration = FMath::Max(0.f, BrakingDecelerationWalking);

	const FVector Velocity2D = Velocity * FVector(1.f, 1.f, 0.f);
	FVector VelocityDir2D;
	float Speed2D;
	Velocity2D.ToDirectionAndLength(VelocityDir2D, Speed2D);

	const float Divisor = ActualBrakingFriction * Speed2D + BrakingDeceleration;
	if (Divisor > 0.f)
	{
		const float TimeToStop = Speed2D / Divisor;
		PredictedStopLocation = Velocity2D * TimeToStop + 0.5f * ((-ActualBrakingFriction) * Velocity2D - BrakingDeceleration * VelocityDir2D) * TimeToStop * TimeToStop;
	}

	return PredictedStopLocation;
}

FVector UAnimCharacterMovementLibrary::PredictGroundMovementPivotLocation(const FVector& Acceleration, const FVector& Velocity, float GroundFriction)
{
	FVector PredictedPivotLocation = FVector::ZeroVector;

	const FVector Acceleration2D = Acceleration * FVector(1.f, 1.f, 0.f);
	
	FVector AccelerationDir2D;
	float AccelerationSize2D;
	Acceleration2D.ToDirectionAndLength(AccelerationDir2D, AccelerationSize2D);

	const float VelocityAlongAcceleration = (Velocity | AccelerationDir2D);
	if (VelocityAlongAcceleration < 0.0f)
	{
		const float SpeedAlongAcceleration = -VelocityAlongAcceleration;
		const float Divisor = AccelerationSize2D + 2.f * SpeedAlongAcceleration * GroundFriction;
		const float TimeToDirectionChange = SpeedAlongAcceleration / Divisor;

		const FVector AccelerationForce = Acceleration - 
			(Velocity - AccelerationDir2D * Velocity.Size2D()) * GroundFriction;
	
		PredictedPivotLocation = Velocity * TimeToDirectionChange + 0.5f * AccelerationForce * TimeToDirectionChange * TimeToDirectionChange;
	}

	return PredictedPivotLocation;
}
