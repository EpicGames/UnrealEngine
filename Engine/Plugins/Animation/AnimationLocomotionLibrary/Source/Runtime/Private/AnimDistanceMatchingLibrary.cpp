// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDistanceMatchingLibrary.h"
#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimSequence.h"
#include "AnimationRuntime.h"
#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "Animation/AnimCurveCompressionCodec_UniformIndexable.h"
#include "SequenceEvaluatorLibrary.h"
#include "SequencePlayerLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDistanceMatchingLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogAnimDistanceMatchingLibrary, Verbose, All);

namespace UE::Anim::DistanceMatchingUtility
{
	static float GetDistanceRange(const UAnimSequenceBase* InAnimSequence, FName CurveName)
	{
		FAnimCurveBufferAccess BufferCurveAccess(InAnimSequence, CurveName);
		if (BufferCurveAccess.IsValid())
		{
			const int32 NumSamples = BufferCurveAccess.GetNumSamples();
			if (NumSamples >= 2)
			{
				return (BufferCurveAccess.GetValue(NumSamples - 1) - BufferCurveAccess.GetValue(0));
			}
		}
		return 0.f;
	}

	static float GetAnimPositionFromDistance(const UAnimSequenceBase* InAnimSequence, const float& InDistance, FName InCurveName)
	{	
		FAnimCurveBufferAccess BufferCurveAccess(InAnimSequence, InCurveName);
		if (BufferCurveAccess.IsValid())
		{
			const int32 NumKeys = BufferCurveAccess.GetNumSamples();
			if (NumKeys < 2)
			{
				return 0.f;
			}

			// Some assumptions: 
			// - keys have unique values, so for a given value, it maps to a single position on the timeline of the animation.
			// - key values are sorted in increasing order.

			int32 First = 1;
			int32 Last = NumKeys - 1;
			int32 Count = Last - First;

			while (Count > 0)
			{
				int32 Step = Count / 2;
				int32 Middle = First + Step;

				if (InDistance > BufferCurveAccess.GetValue(Middle))
				{
					First = Middle + 1;
					Count -= Step + 1;
				}
				else
				{
					Count = Step;
				}
			}

			const float KeyAValue = BufferCurveAccess.GetValue(First - 1);
			const float KeyBValue = BufferCurveAccess.GetValue(First);
			const float Diff = KeyBValue - KeyAValue;
			const float Alpha = !FMath::IsNearlyZero(Diff) ? ((InDistance - KeyAValue) / Diff) : 0.f;

			const float KeyATime = BufferCurveAccess.GetTime(First - 1);
			const float KeyBTime = BufferCurveAccess.GetTime(First);
			return FMath::Lerp(KeyATime, KeyBTime, Alpha);
		}

		return 0.f;
	}

	/**
	* Advance from the current time to a new time in the animation that will result in the desired distance traveled by the authored root motion.
	*/
	static float GetTimeAfterDistanceTraveled(const UAnimSequenceBase* AnimSequence, float CurrentTime, float DistanceTraveled, FName CurveName, const bool bAllowLooping)
	{
		float NewTime = CurrentTime;
		if (AnimSequence != nullptr)
		{
			// Avoid infinite loops if the animation doesn't cover any distance.
			if (!FMath::IsNearlyZero(UE::Anim::DistanceMatchingUtility::GetDistanceRange(AnimSequence, CurveName)))
			{
				float AccumulatedDistance = 0.f;
				float AccumulatedTime = 0.f;

				const float SequenceLength = AnimSequence->GetPlayLength();
				static const float StepTime = 1.f / 30.f;				

				FAnimExtractContext Context;
				// Traverse the distance curve, accumulating animated distance until the desired distance is reached.
				while ((AccumulatedDistance < DistanceTraveled) && (bAllowLooping || (NewTime + StepTime < SequenceLength)))
				{
					Context.CurrentTime = NewTime;
					const float CurrentDistance = AnimSequence->EvaluateCurveData(CurveName, Context);
					Context.CurrentTime = NewTime + StepTime;
					const float DistanceAfterStep = AnimSequence->EvaluateCurveData(CurveName, Context);
					const float AnimationDistanceThisStep = DistanceAfterStep - CurrentDistance;

					if (!FMath::IsNearlyZero(AnimationDistanceThisStep))
					{
						// Keep advancing if the desired distance hasn't been reached.
						if (AccumulatedDistance + AnimationDistanceThisStep < DistanceTraveled)
						{
							FAnimationRuntime::AdvanceTime(bAllowLooping, StepTime, NewTime, SequenceLength);
							AccumulatedDistance += AnimationDistanceThisStep;
						}
						// Once the desired distance is passed, find the approximate time between samples where the distance will be reached.
						else
						{
							const float DistanceAlpha = (DistanceTraveled - AccumulatedDistance) / AnimationDistanceThisStep;
							FAnimationRuntime::AdvanceTime(bAllowLooping, DistanceAlpha * StepTime, NewTime, SequenceLength);
							AccumulatedDistance = DistanceTraveled;
							break;
						}
					}
					else
					{
						NewTime += StepTime;
					}
					
					AccumulatedTime += StepTime;

					// If the animation doesn't cover enough distance, we abandon the algorithm to avoid an infinite loop.
					if (AccumulatedTime >= SequenceLength)
					{
						UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Failed to advance distance of (%.2f) after (%.2f) seconds on anim sequence (%ls). Aborting.", 
							DistanceTraveled, AccumulatedTime, *GetNameSafe(AnimSequence));
						break;
					}
				}
			}
			else
			{
				UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Anim sequence (%ls) is missing a distance curve or doesn't cover enough distance for GetTimeAfterDistanceTraveled.", *GetNameSafe(AnimSequence));
			}
		}
		else
		{
			UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Invalid AnimSequence passed to GetTimeAfterDistanceTraveled");
		}

		return NewTime;
	}
}

FSequenceEvaluatorReference UAnimDistanceMatchingLibrary::AdvanceTimeByDistanceMatching(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator,
	float DistanceTraveled, FName DistanceCurveName, FVector2D PlayRateClamp)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("AdvanceTimeByDistanceMatching"),
		[&UpdateContext, DistanceTraveled, DistanceCurveName, PlayRateClamp](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if (const FAnimationUpdateContext* AnimationUpdateContext = UpdateContext.GetContext())
			{
				const float DeltaTime = AnimationUpdateContext->GetDeltaTime(); 

				if (DeltaTime > 0 && DistanceTraveled > 0)
				{
					if (const UAnimSequenceBase* AnimSequence = Cast<UAnimSequence>(InSequenceEvaluator.GetSequence()))
					{
						const float CurrentTime = InSequenceEvaluator.GetExplicitTime();
						const float CurrentAssetLength = InSequenceEvaluator.GetCurrentAssetLength();
						const bool bAllowLooping = InSequenceEvaluator.IsLooping();

						float TimeAfterDistanceTraveled = UE::Anim::DistanceMatchingUtility::GetTimeAfterDistanceTraveled(AnimSequence, CurrentTime, DistanceTraveled, DistanceCurveName, bAllowLooping);

						// Calculate the effective playrate that would result from advancing the animation by the distance traveled.
						// Account for the animation looping.
						if (TimeAfterDistanceTraveled < CurrentTime)
						{
							TimeAfterDistanceTraveled += CurrentAssetLength;
						}
						float EffectivePlayRate = (TimeAfterDistanceTraveled - CurrentTime) / DeltaTime;

						// Clamp the effective play rate.
						if (PlayRateClamp.X >= 0.0f && PlayRateClamp.X < PlayRateClamp.Y)
						{
							EffectivePlayRate = FMath::Clamp(EffectivePlayRate, PlayRateClamp.X, PlayRateClamp.Y);
						}

						// Advance animation time by the effective play rate.
						float NewTime = CurrentTime;
						FAnimationRuntime::AdvanceTime(bAllowLooping, EffectivePlayRate * DeltaTime, NewTime, CurrentAssetLength);

						if (!InSequenceEvaluator.SetExplicitTime(NewTime))
						{
							UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Could not set explicit time on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
						}
					}
					else
					{
						UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Sequence evaluator does not have an anim sequence to play.");
					}
				}
			}
			else
			{
				UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "AdvanceTimeByDistanceMatching called with invalid context");
			}
		});

	return SequenceEvaluator;
}

FSequenceEvaluatorReference UAnimDistanceMatchingLibrary::DistanceMatchToTarget(const FSequenceEvaluatorReference& SequenceEvaluator,
	float DistanceToTarget, FName DistanceCurveName)
{
	SequenceEvaluator.CallAnimNodeFunction<FAnimNode_SequenceEvaluator>(
		TEXT("DistanceMatchToTarget"),
		[DistanceToTarget, DistanceCurveName](FAnimNode_SequenceEvaluator& InSequenceEvaluator)
		{
			if (const UAnimSequenceBase* AnimSequence = Cast<UAnimSequence>(InSequenceEvaluator.GetSequence()))
			{
				// By convention, distance curves store the distance to a target as a negative value.
				const float NewTime = UE::Anim::DistanceMatchingUtility::GetAnimPositionFromDistance(AnimSequence, -DistanceToTarget, DistanceCurveName);
				if (!InSequenceEvaluator.SetExplicitTime(NewTime))
				{
					UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Could not set explicit time on sequence evaluator, value is not dynamic. Set it as Always Dynamic.");
				}
			}
			else
			{
				UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Sequence evaluator does not have an anim sequence to play.");
			}
			
		});

	return SequenceEvaluator;
}

FSequencePlayerReference UAnimDistanceMatchingLibrary::SetPlayrateToMatchSpeed(const FSequencePlayerReference& SequencePlayer, float SpeedToMatch, FVector2D PlayRateClamp)
{
	SequencePlayer.CallAnimNodeFunction<FAnimNode_SequencePlayer>(
		TEXT("SetPlayrateToMatchSpeed"),
		[SpeedToMatch, PlayRateClamp](FAnimNode_SequencePlayer& InSequencePlayer)
		{
			if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(InSequencePlayer.GetSequence()))
			{
				const float AnimLength = AnimSequence->GetPlayLength();
				if (!FMath::IsNearlyZero(AnimLength))
				{
					// Calculate the speed as: (distance traveled by the animation) / (length of the animation)
					const FVector RootMotionTranslation = AnimSequence->ExtractRootMotionFromRange(0.0f, AnimLength, FAnimExtractContext()).GetTranslation();
					const float RootMotionDistance = RootMotionTranslation.Size2D();
					if (!FMath::IsNearlyZero(RootMotionDistance))
					{
						const float AnimationSpeed = RootMotionDistance / AnimLength;
						float DesiredPlayRate = SpeedToMatch / AnimationSpeed;
						if (PlayRateClamp.X >= 0.0f && PlayRateClamp.X < PlayRateClamp.Y)
						{
							DesiredPlayRate = FMath::Clamp(DesiredPlayRate, PlayRateClamp.X, PlayRateClamp.Y);
						}

						if (!InSequencePlayer.SetPlayRate(DesiredPlayRate))
						{
							UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Could not set play rate on sequence player, value is not dynamic. Set it as Always Dynamic.");
						}
					}
					else
					{
						UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Unable to adjust playrate for animation with no root motion delta (%ls).", *GetNameSafe(AnimSequence));
					}
				}
				else
				{
					UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Unable to adjust playrate for zero length animation (%ls).", *GetNameSafe(AnimSequence));
				}
			}
			else
			{
				UE_LOGF(LogAnimDistanceMatchingLibrary, Warning, "Sequence player does not have an anim sequence to play.");
			}
		});

	return SequencePlayer;
}
