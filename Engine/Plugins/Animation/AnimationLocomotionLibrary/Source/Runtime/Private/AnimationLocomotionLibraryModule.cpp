// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationLocomotionLibraryModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AnimationLocomotionModule"

void FAnimationLocomotionLibraryRuntimeModule::StartupModule()
{

}

void FAnimationLocomotionLibraryRuntimeModule::ShutdownModule()
{

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAnimationLocomotionLibraryRuntimeModule, AnimationLocomotionLibraryRuntime)
