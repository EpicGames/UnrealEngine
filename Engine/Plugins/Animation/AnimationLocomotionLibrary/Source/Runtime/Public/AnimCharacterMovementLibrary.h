// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AnimCharacterMovementLibrary.generated.h"

#define UE_API ANIMATIONLOCOMOTIONLIBRARYRUNTIME_API

class UAnimSequence;

/**
  * Library of common techniques for driving character locomotion animations. 
  */
UCLASS(MinimalAPI)
class UAnimCharacterMovementLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Predict where the character will stop based on its current movement properties and parameters from the movement component.
	 * This uses prediction logic that is heavily tied to the UCharacterMovementComponent.
	 * Each parameter corresponds to a value from the UCharacterMovementComponent with the same name.
	 * Because this is a thread safe function, it's	recommended to populate these fields via the Property Access system.
	 * @return The predicted stop position in local space to the character. The size of this vector will be the distance to the stop location.
	 */
	UFUNCTION(BlueprintPure, Category = "Animation Character Movement", meta = (BlueprintThreadSafe))
	static UE_API FVector PredictGroundMovementStopLocation(const FVector& Velocity,
		bool bUseSeparateBrakingFriction, float BrakingFriction, float GroundFriction, float BrakingFrictionFactor, float BrakingDecelerationWalking);

	/**
	 * Predict where the character will change direction during a pivot based on its current movement properties and parameters from the movement component.
	 * This uses prediction logic that is heavily tied to the UCharacterMovementComponent.
	 * Each parameter corresponds to a value from the UCharacterMovementComponent with the same name.
	 * Because this is a thread safe function, it's	recommended to populate these fields via the Property Access system.
	 * @return The predicted pivot position in local space to the character. The size of this vector will be the distance to the pivot.
	 */
	UFUNCTION(BlueprintPure, Category = "Animation Character Movement", meta = (BlueprintThreadSafe))
	static UE_API FVector PredictGroundMovementPivotLocation(const FVector& Acceleration, const FVector& Velocity, float GroundFriction);
};

#undef UE_API
