// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceEvaluatorLibrary.h"
#include "SequencePlayerLibrary.h"
#include "Animation/AnimCurveTypes.h"
#include "AnimDistanceMatchingLibrary.generated.h"

#define UE_API ANIMATIONLOCOMOTIONLIBRARYRUNTIME_API

class UAnimSequence;

/**
 * Library of techniques for driving animations by distance metrics rather than by time.
 * These techniques can be effective at compensating for differences between character movement and authored motion in the animations.
 * Distance Matching effectively changes the play rate of the animation to keep the feet from sliding. It's common to clamp the resulting
 * play rate to avoid animations playing too slow or too fast and to use techniques such as Stride Warping to make up the difference.
 */
UCLASS(MinimalAPI)
class UAnimDistanceMatchingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Advance the sequence evaluator forward by distance traveled rather than time. A distance curve is required on the animation that
	 * describes the distance traveled by the root bone in the animation. See UDistanceCurveModifier.
	 * @param UpdateContext - The update context provided in the anim node function.
	 * @param SequenceEvaluator - The sequence evaluator node to operate on.
	 * @param DistanceTraveled - The distance traveled by the character since the last animation update.
	 * @param DistanceCurveName - Name of the curve we want to match 
	 * @param PlayRateClamp - A clamp on the effective play rate of the animation after distance matching. Set to (0,0) for no clamping.
	 */
	UFUNCTION(BlueprintCallable, Category = "Distance Matching", meta=(BlueprintThreadSafe))
	static UE_API FSequenceEvaluatorReference AdvanceTimeByDistanceMatching(const FAnimUpdateContext& UpdateContext, const FSequenceEvaluatorReference& SequenceEvaluator,
		float DistanceTraveled, FName DistanceCurveName, FVector2D PlayRateClamp = FVector2D(0.75f, 1.25f));

	/**
	 * Set the time of the sequence evaluator to the point in the animation where the distance curve matches the DistanceToTarget input.
	 * A common use case is to achieve stops without foot sliding by, each frame, selecting the point in the animation that matches the distance the character has remaining until it stops.
	 * Note that because this technique sets the time of the animation by distance remaining, it doesn't respect phase of any previous animation (e.g. from a jog cycle).
	 * @param SequenceEvaluator - The sequence evaluator node to operate on.
	 * @param DistanceToTarget - The distance remaining to a target (e.g. a stop or pivot point).
	 * @param DistanceCurveName - Name of the curve we want to match 
	 */
	UFUNCTION(BlueprintCallable, Category = "Distance Matching", meta=(BlueprintThreadSafe))
	static UE_API FSequenceEvaluatorReference DistanceMatchToTarget(const FSequenceEvaluatorReference& SequenceEvaluator,
		float DistanceToTarget, FName DistanceCurveName);

	/**
	 * Set the play rate of the sequence player so that the speed of the animation matches in-game movement speed.
	 * While distance matching is commonly used for transition animations, cycle animations (walk, jog, etc) typically just adjust their play rate to match
	 * the in-game movement speed.
	 * This function assumes that the animation has a constant speed.
	 * @param SequencePlayer - The sequence player node to operate on.
	 * @param SpeedToMatch - The in-game movement speed to match. This is usually the current speed of the movement component.
	 * @param PlayRateClamp - A clamp on how much the animation's play rate can change to match the in-game movement speed. Set to (0,0) for no clamping.
	 */
	UFUNCTION(BlueprintCallable, Category = "Distance Matching", meta=(BlueprintThreadSafe))
	static UE_API FSequencePlayerReference SetPlayrateToMatchSpeed(const FSequencePlayerReference& SequencePlayer, 
		float SpeedToMatch, FVector2D PlayRateClamp = FVector2D(0.75f, 1.25f));
};

#undef UE_API
